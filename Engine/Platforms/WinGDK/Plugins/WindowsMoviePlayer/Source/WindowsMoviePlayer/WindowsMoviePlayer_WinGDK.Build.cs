// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class WindowsMoviePlayer_WinGDK : WindowsMoviePlayer
	{
		public WindowsMoviePlayer_WinGDK( ReadOnlyTargetRules Target) : base(Target)
		{
		}
	}
}
