// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsMoviePlayer.h"
#include "MoviePlayer.h"
#include "WindowsMovieStreamer.h"
#include "Modules/ModuleManager.h"

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <mfapi.h>
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"

#include "Misc/CoreDelegates.h"

TSharedPtr<FMediaFoundationMovieStreamer, ESPMode::ThreadSafe> MovieStreamer;

class FWindowsMoviePlayerModule : public IModuleInterface
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
		bool bLoadSuccessful = true;
		// now attempt to load the delay loaded DLLs
		if (!LoadMediaLibrary(TEXT("shlwapi.dll")))
		{
			bLoadSuccessful = false;
		}
		if (!LoadMediaLibrary(TEXT("mf.dll")))
		{
			bLoadSuccessful = false;
		}
		if (!LoadMediaLibrary(TEXT("mfplat.dll")))
		{
			bLoadSuccessful = false;
		}
		if (!LoadMediaLibrary(TEXT("mfplay.dll")))
		{
			bLoadSuccessful = false;
		}

		if( bLoadSuccessful )
		{
			HRESULT Hr = MFStartup(MF_VERSION);
			check(SUCCEEDED(Hr));

            MovieStreamer = MakeShareable(new FMediaFoundationMovieStreamer);
            FCoreDelegates::RegisterMovieStreamerDelegate.Broadcast(MovieStreamer);
		}
	}

	virtual void ShutdownModule() override
	{
        if( MovieStreamer.IsValid())
		{
            FCoreDelegates::UnRegisterMovieStreamerDelegate.Broadcast(MovieStreamer);
            MovieStreamer.Reset();
            MFShutdown();
		}
	}

	bool LoadMediaLibrary(const FString& Library)
	{
		if (LoadLibraryW(*Library) == NULL)
		{
			uint32 ErrorCode = GetLastError();
			if (ErrorCode == ERROR_MOD_NOT_FOUND)
			{
				UE_LOGF(LogWindowsMoviePlayer, Log, "Could not load %ls. Library not found.", *Library);
			}
			else
			{
				UE_LOGF(LogWindowsMoviePlayer, Warning, "Could not load %ls. Error=%d", *Library, ErrorCode);
			}
			
			return false;
		}

		return true;
	}
};

IMPLEMENT_MODULE( FWindowsMoviePlayerModule, WindowsMoviePlayer )
