// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsMovieStreamer.h"

#include "IHeadMountedDisplayModule.h"
#include "MoviePlayer.h"
#include "RenderUtils.h"
#include "Rendering/RenderingCommon.h"
#include "Slate/SlateTextures.h"

#include "MediaShaders.h"
#include "PipelineStateCache.h"
#include "RHIStaticStates.h"

#pragma comment(lib, "shlwapi")
#pragma comment(lib, "mf")
#pragma comment(lib, "mfplat")
#pragma comment(lib, "mfplay")
#pragma comment(lib, "mfuuid")

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Microsoft/COMPointer.h"

#include <windows.h>
#include <shlwapi.h>
#include <mfapi.h>
#include <mfidl.h>
#include <mferror.h>


DEFINE_LOG_CATEGORY(LogWindowsMoviePlayer);

FMediaFoundationMovieStreamer::FMediaFoundationMovieStreamer()
	: TextureData()
	, VideoPlayer(NULL)
	, SampleGrabberCallback(NULL)
	, bUseSound(true)
{
	MovieViewport = MakeShareable(new FMovieViewport());
	PlaybackType = MT_Normal;
}

FMediaFoundationMovieStreamer::~FMediaFoundationMovieStreamer()
{
	CloseMovie();
	CleanupRenderingResources();

	FlushRenderingCommands();

	TextureFreeList.Empty();
}

bool FMediaFoundationMovieStreamer::Init(const TArray<FString>& MoviePaths, TEnumAsByte<EMoviePlaybackType> inPlaybackType)
{
	if (MoviePaths.Num() == 0)
	{
		return false;
	}

	MovieIndex = 0;
	PlaybackType = inPlaybackType;
	StoredMoviePaths = MoviePaths;

	MovieViewport->SetTexture(nullptr);

	return OpenNextMovie();
}

void FMediaFoundationMovieStreamer::ForceCompletion()
{
	CloseMovie();
}

void FMediaFoundationMovieStreamer::ConvertSample(FRHICommandList& RHICmdList)
{
	const bool SrgbOutput = false;
	const bool bSampleIsOutputSrgb = false;

	const FMovieTrackFormat& SourceFormat = VideoPlayer->GetVideoTrackFormat();
	const EPixelFormat InputPixelFormat = PF_B8G8R8A8;

	{
		const bool SrgbTexture = false;
		const ETextureCreateFlags InputCreateFlags = SrgbTexture ? TexCreate_SRGB : TexCreate_None;

		// create a new input render target if necessary
		if (!InputTarget.IsValid() || (InputTarget->GetSizeXY() != SourceFormat.BufferDim) || (InputTarget->GetFormat() != InputPixelFormat) || ((InputTarget->GetFlags() & TexCreate_SRGB) != InputCreateFlags))
		{
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("FMediaFoundationMovieStreamer"))
				.SetExtent(SourceFormat.BufferDim)
				.SetFormat(InputPixelFormat)
				.SetFlags(InputCreateFlags | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
				.SetInitialState(ERHIAccess::SRVMask);

			InputTarget = RHICmdList.CreateTexture(Desc);
		}

		// copy sample data to input render target
		FUpdateTextureRegion2D Region(0, 0, 0, 0, SourceFormat.BufferDim.X, SourceFormat.BufferDim.Y);
		RHICmdList.UpdateTexture2D(InputTarget, 0, Region, SourceFormat.BufferStride, TextureData.GetData());
	}

	const FIntPoint OutputDim = SourceFormat.OutputDim;

	FSlateTexture2DRHIRef* CurrentTexture = Texture.Get();
	FRHITexture* RenderTarget = CurrentTexture->GetRHIRef();

	// perform the conversion
	RHICmdList.Transition(FRHITransitionInfo(RenderTarget, ERHIAccess::Unknown, ERHIAccess::RTV));

	FRHIRenderPassInfo RPInfo(RenderTarget, ERenderTargetActions::Load_Store);
	RHICmdList.BeginRenderPass(RPInfo, TEXT("WindowsMovieConvertSample"));
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit;

		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		RHICmdList.SetViewport(0, 0, 0.0f, OutputDim.X, OutputDim.Y, 1.0f);

		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
		GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

		// configure media shaders
		auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FMediaShadersVS> VertexShader(ShaderMap);

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

		bool bValidFormat = true;
		switch (SourceFormat.SampleFormat)
		{
		case EMediaTextureSampleFormat::CharBMP:
		{
			TShaderMapRef<FBMPConvertPS> ConvertShader(ShaderMap);
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
			SetShaderParametersLegacyPS(RHICmdList, ConvertShader, InputTarget, OutputDim, bSampleIsOutputSrgb && !SrgbOutput);
		}
		break;

		case EMediaTextureSampleFormat::CharYUY2:
		{
			TShaderMapRef<FYUY2ConvertPS> ConvertShader(ShaderMap);
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
			SetShaderParametersLegacyPS(RHICmdList, ConvertShader, InputTarget, OutputDim, MediaShaders::YuvToRgbRec709Scaled, MediaShaders::YUVOffset8bits, bSampleIsOutputSrgb);
		}
		break;

		default:
			bValidFormat = false; // unsupported format
		}

		if (bValidFormat)
		{
			// draw full size quad into render target
			FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(RHICmdList);
			RHICmdList.SetStreamSource(0, VertexBuffer, 0);
			// set viewport to RT size
			RHICmdList.SetViewport(0, 0, 0.0f, OutputDim.X, OutputDim.Y, 1.0f);
			RHICmdList.DrawPrimitive(0, 2, 1);
		}
	}
	RHICmdList.EndRenderPass();
	RHICmdList.Transition(FRHITransitionInfo(RenderTarget, ERHIAccess::Unknown, ERHIAccess::SRVGraphics));
}

bool FMediaFoundationMovieStreamer::Tick(FRHICommandListBase& RHICmdList, float DeltaTime)
{
	FSlateTexture2DRHIRef* CurrentTexture = Texture.Get();

	if (CurrentTexture && !CurrentTexture->IsInitialized())
	{
		CurrentTexture->InitResource(RHICmdList);
	}

	if (CurrentTexture && SampleGrabberCallback->GetIsSampleReadyToUpdate())
	{
		const FMovieTrackFormat& SourceFormat = VideoPlayer->GetVideoTrackFormat();
		if (SourceFormat.SampleFormat == EMediaTextureSampleFormat::CharBGRA && SourceFormat.BufferDim == SourceFormat.OutputDim)
		{
			uint32 Stride;
			uint8* DestTextureData = (uint8*)RHICmdList.GetAsImmediate().LockTexture2D( CurrentTexture->GetTypedResource(), 0, RLM_WriteOnly, Stride, false );
			FMemory::Memcpy( DestTextureData, TextureData.GetData(), TextureData.Num());
			RHICmdList.GetAsImmediate().UnlockTexture2D( CurrentTexture->GetTypedResource(), 0, false );
		}
		else
		{
			ConvertSample(RHICmdList.GetAsImmediate());
		}

		if (MovieViewport->GetViewportRenderTargetTexture() == nullptr)
		{
			MovieViewport->SetTexture(Texture);
		}

		SampleGrabberCallback->SetNeedNewSample();
	}

	if (!VideoPlayer->MovieIsRunning())
	{
		// Playback can fail when no audio output devices enabled
		if (VideoPlayer->FailedToCreateMediaSink() && bUseSound)
		{
			// Retry playback without audio
			bUseSound = false;
			--MovieIndex;
		}

		CloseMovie();
		if ((MovieIndex + 1) < StoredMoviePaths.Num())
		{
			++MovieIndex;
			if (OpenNextMovie())
			{
				MovieViewport->SetTexture(Texture);
			}
		}
		else if (PlaybackType != MT_Normal)
		{
			MovieIndex = PlaybackType == MT_LoadingLoop ? StoredMoviePaths.Num() - 1 : 0;
			if (OpenNextMovie())
			{
				MovieViewport->SetTexture(Texture);
			}
		}
		else
		{
			return true;
		}
	}

	return false;
}

FString FMediaFoundationMovieStreamer::GetMovieName()
{
	return StoredMoviePaths.IsValidIndex(MovieIndex) ? StoredMoviePaths[MovieIndex] : TEXT("");
}

bool FMediaFoundationMovieStreamer::IsLastMovieInPlaylist()
{
	return MovieIndex == (StoredMoviePaths.Num() - 1);
}


void FMediaFoundationMovieStreamer::Cleanup()
{
	CleanupRenderingResources();
}

bool FMediaFoundationMovieStreamer::OpenNextMovie()
{
	check(StoredMoviePaths.Num() > 0 && MovieIndex < StoredMoviePaths.Num());
	FString MoviePath = FPaths::ProjectContentDir() + TEXT("Movies/") + StoredMoviePaths[MovieIndex];

	SampleGrabberCallback = new FSampleGrabberCallback(TextureData);

	VideoPlayer = new FVideoPlayer();
	FIntPoint VideoDimensions = VideoPlayer->OpenFile(MoviePath, SampleGrabberCallback, bUseSound);

	if( VideoDimensions != FIntPoint::ZeroValue )
	{
		TextureData.Empty();

		if( TextureFreeList.Num() > 0 )
		{
			Texture = TextureFreeList.Pop();

			if( Texture->GetWidth() != VideoDimensions.X || Texture->GetHeight() != VideoDimensions.Y )
			{
				FSlateTexture2DRHIRef* TextureRHIRef = Texture.Get();
				ENQUEUE_RENDER_COMMAND(UpdateMovieTexture)(
					[TextureRHIRef, VideoDimensions](FRHICommandListImmediate& RHICmdList)
					{
						TextureRHIRef->Resize(VideoDimensions.X, VideoDimensions.Y);
					});
			}
		}
		else
		{
			const bool bCreateEmptyTexture = true;
			Texture = MakeShareable(new FSlateTexture2DRHIRef(VideoDimensions.X, VideoDimensions.Y, PF_B8G8R8A8, NULL, TexCreate_RenderTargetable, bCreateEmptyTexture));

			FSlateTexture2DRHIRef* TextureRHIRef = Texture.Get();
			ENQUEUE_RENDER_COMMAND(InitMovieTexture)(
				[TextureRHIRef](FRHICommandListImmediate& RHICmdList)
				{
					TextureRHIRef->InitResource(RHICmdList);
				});
		}

		VideoPlayer->StartPlayback();
		return true;
	}
	else
	{
		return false;
	}
}

void FMediaFoundationMovieStreamer::CloseMovie()
{
	BroadcastCurrentMovieClipFinished(GetMovieName());

	if (Texture.IsValid())
	{
		TextureFreeList.Add(Texture);

		MovieViewport->SetTexture(NULL);
		Texture.Reset();
	}

	if (VideoPlayer)
	{
		VideoPlayer->Shutdown();
		VideoPlayer->Release();
		VideoPlayer = NULL;
	}
	if (SampleGrabberCallback)
	{
		SampleGrabberCallback->Release();
		SampleGrabberCallback = NULL;
	}
}

void FMediaFoundationMovieStreamer::CleanupRenderingResources()
{
	for( int32 TextureIndex = 0; TextureIndex < TextureFreeList.Num(); ++TextureIndex )
	{
		BeginReleaseResource( TextureFreeList[TextureIndex].Get() );
	}

	InputTarget.SafeRelease();
}

#if _MSC_VER == 1900
#pragma warning(push)
#pragma warning(disable:4838)
#endif // _MSC_VER == 1900
STDMETHODIMP FVideoPlayer::QueryInterface(REFIID RefID, void** Object)
{
	static const QITAB QITab[] =
	{
		QITABENT(FVideoPlayer, IMFAsyncCallback),
		{ 0 }
	};
	return QISearch(this, QITab, RefID, Object);
}
#if _MSC_VER == 1900
#pragma warning(pop)
#endif // _MSC_VER == 1900

STDMETHODIMP_(ULONG) FVideoPlayer::AddRef()
{
	return FPlatformAtomics::InterlockedIncrement(&RefCount);
}

STDMETHODIMP_(ULONG) FVideoPlayer::Release()
{
    int32 Ref = FPlatformAtomics::InterlockedDecrement(&RefCount);
	if (Ref == 0) {delete this;}
	return Ref;
}

HRESULT FVideoPlayer::Invoke(IMFAsyncResult* AsyncResult)
{
	TComPtr<IMFMediaEvent> Event;
	HRESULT HResult = MediaSession->EndGetEvent(AsyncResult, &Event);
	if (FAILED(HResult))
	{
		return S_OK;
	}

	// get event type
	MediaEventType EventType = MEUnknown;
	HResult = Event->GetType(&EventType);
	if (FAILED(HResult))
	{
		return S_OK;
	}

	// get event status
	HRESULT EventStatus = S_FALSE;
	HResult = Event->GetStatus(&EventStatus);
	if (FAILED(HResult))
	{
		return S_OK;
	}

	bool bRequestNextEvent = true;
	bool bFinishedAndClose = false;
	switch (EventType)
	{
	case MESinkInvalidated:
		// Need to stop playback now or will be stuck forever
		bFinishedAndClose = true;
		break;

	case MESessionClosed:
		bFinishedAndClose = true;
		break;

	case MESessionTopologySet:
		if (!SUCCEEDED(EventStatus))
		{
			if (EventStatus == MF_E_CANNOT_CREATE_SINK)
			{
				bFailedToCreateMediaSink = true;
			}

			// Topology error
			bFinishedAndClose = true;
		}
		break;

	case MEEndOfPresentation:
		if (MovieIsRunning())
		{
			MovieIsFinished.Set(1);
		}
		break;

	case MEError:
		if (MovieIsRunning())
		{
			// Unknown fatal error
			bFinishedAndClose = true;
		}
		break;

	default:
		break;
	}

	if (bFinishedAndClose)
	{
		MovieIsFinished.Set(1);
		CloseIsPosted.Set(1);
		bRequestNextEvent = false;
	}

	if (bRequestNextEvent)
	{
		HResult = MediaSession->BeginGetEvent(this, NULL);
		if (FAILED(HResult))
		{
			MovieIsFinished.Set(1);
			CloseIsPosted.Set(1);
			return S_OK;
		}
	}

	return S_OK;
}

FIntPoint FVideoPlayer::OpenFile(const FString& FilePath, class FSampleGrabberCallback* SampleGrabberCallback, bool bUseSound)
{
	FIntPoint OutDimensions = FIntPoint::ZeroValue;

	HRESULT HResult = S_OK;

	{
		HResult = MFCreateMediaSession(NULL, &MediaSession);
		check(SUCCEEDED(HResult));

		HResult = MediaSession->BeginGetEvent(this, NULL);
		check(SUCCEEDED(HResult));
	}

	{
		IMFSourceResolver* SourceResolver = NULL;
		IUnknown* Source = NULL;
		HResult = MFCreateSourceResolver(&SourceResolver);
		check(SUCCEEDED(HResult));

		// Assume MP4 for now.
		FString PathPlusExt = FString::Printf( TEXT("%s.%s"), *FilePath, TEXT("mp4") );

		MF_OBJECT_TYPE ObjectType = MF_OBJECT_INVALID;
		HResult = SourceResolver->CreateObjectFromURL(*PathPlusExt, MF_RESOLUTION_MEDIASOURCE, NULL, &ObjectType, &Source);
		SourceResolver->Release();
		if( SUCCEEDED(HResult ) )
		{
			HResult = Source->QueryInterface(IID_PPV_ARGS(&MediaSource));
			Source->Release();

			OutDimensions = SetPlaybackTopology(SampleGrabberCallback, bUseSound);
		}
		else
		{
			UE_LOGF(LogWindowsMoviePlayer, Log, "Unable to load movie: %ls", *PathPlusExt );
			MovieIsFinished.Set(1);
		}

	}

	return OutDimensions;
}

void FVideoPlayer::StartPlayback()
{
	check(MediaSession != NULL);

	PROPVARIANT VariantStart;
	PropVariantInit(&VariantStart);

	HRESULT HResult = MediaSession->Start(&GUID_NULL, &VariantStart);
	check(SUCCEEDED(HResult));
	PropVariantClear(&VariantStart);
}

void FVideoPlayer::Shutdown()
{
	if (MediaSession)
	{
		HRESULT HResult = MediaSession->Close();
		if (SUCCEEDED(HResult))
		{
			while (CloseIsPosted.GetValue() == 0)
			{
				FPlatformProcess::Sleep(0.010f);
			}
		}
	}

	if (MediaSource)
	{
		MediaSource->Shutdown();
		MediaSource->Release();
		MediaSource = NULL;
	}
	if (MediaSession)
	{
		MediaSession->Shutdown();
		MediaSession->Release();
		MediaSession = NULL;
	}
}

FIntPoint FVideoPlayer::SetPlaybackTopology(FSampleGrabberCallback* SampleGrabberCallback, bool bUseSound)
{
	FIntPoint OutDimensions = FIntPoint(ForceInit);

	HRESULT HResult = S_OK;

	IMFPresentationDescriptor* PresentationDesc = NULL;
	HResult = MediaSource->CreatePresentationDescriptor(&PresentationDesc);
	check(SUCCEEDED(HResult));

	IMFTopology* Topology = NULL;
	HResult = MFCreateTopology(&Topology);
	check(SUCCEEDED(HResult));

	DWORD StreamCount = 0;
	HResult = PresentationDesc->GetStreamDescriptorCount(&StreamCount);
	check(SUCCEEDED(HResult));

	for (uint32 i = 0; i < StreamCount; ++i)
	{
		BOOL bSelected = 0;
		IMFStreamDescriptor* StreamDesc = NULL;
		HResult = PresentationDesc->GetStreamDescriptorByIndex(i, &bSelected, &StreamDesc);
		check(SUCCEEDED(HResult));

		if (bSelected)
		{
			FIntPoint VideoDimensions = AddStreamToTopology(Topology, PresentationDesc, StreamDesc, SampleGrabberCallback, bUseSound);
			if (VideoDimensions != FIntPoint(ForceInit))
			{
				OutDimensions = VideoDimensions;
			}
		}

		StreamDesc->Release();
	}

	HResult = MediaSession->SetTopology(0, Topology);
	check(SUCCEEDED(HResult));

	Topology->Release();
	PresentationDesc->Release();

	return OutDimensions;
}

FIntPoint FVideoPlayer::AddStreamToTopology(IMFTopology* Topology, IMFPresentationDescriptor* PresentationDesc, IMFStreamDescriptor* StreamDesc, FSampleGrabberCallback* SampleGrabberCallback, bool bUseSound)
{
	FIntPoint OutDimensions = FIntPoint(ForceInit);

	HRESULT HResult = S_OK;

	IMFActivate* SinkActivate = NULL;
	{
		IMFMediaTypeHandler* Handler = NULL;
		HResult = StreamDesc->GetMediaTypeHandler(&Handler);
		check(SUCCEEDED(HResult));

		GUID MajorType;
		HResult = Handler->GetMajorType(&MajorType);
		check(SUCCEEDED(HResult));

		if (MajorType == MFMediaType_Audio)
		{
			if (!bUseSound)
			{
				SAFE_RELEASE(Handler);
				return FIntPoint(ForceInit);
			}

			HResult = MFCreateAudioRendererActivate(&SinkActivate);
			check(SUCCEEDED(HResult));

			// Allow HMD, if present, to override audio output device
			// Can only access HMD from the game thread.
			if (IsInGameThread())
			{
				if (IHeadMountedDisplayModule::IsAvailable())
				{
					HMDAudioOutputDevice = IHeadMountedDisplayModule::Get().GetAudioOutputDevice();
				}
				else
				{
					HMDAudioOutputDevice.Reset();
				}
			}
			if(!HMDAudioOutputDevice.IsEmpty())
			{
				HResult = SinkActivate->SetString(MF_AUDIO_RENDERER_ATTRIBUTE_ENDPOINT_ID, *HMDAudioOutputDevice);
				check(SUCCEEDED(HResult));
			}
		}
		else if (MajorType == MFMediaType_Video)
		{

			IMFMediaType* OutputType = NULL;
			HResult = Handler->GetCurrentMediaType(&OutputType);
			check(SUCCEEDED(HResult));

			IMFMediaType* InputType = NULL;
			HResult = MFCreateMediaType(&InputType);
			check(SUCCEEDED(HResult));
			UINT32 Width = 0, Height = 0;
			HResult = MFGetAttributeSize(OutputType, MF_MT_FRAME_SIZE, &Width, &Height);
			check(SUCCEEDED(HResult));

			GUID SourceVideoSubType;
			HResult = OutputType->GetGUID(MF_MT_SUBTYPE, &SourceVideoSubType);
			check(SUCCEEDED(HResult));

			HResult = InputType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Video);
			check(SUCCEEDED(HResult));

			VideoTrackFormat.OutputDim = FIntPoint(Width, Height);
			{
				const bool Uncompressed =
					(SourceVideoSubType == MFVideoFormat_RGB555) ||
					(SourceVideoSubType == MFVideoFormat_RGB565) ||
					(SourceVideoSubType == MFVideoFormat_RGB24) ||
					(SourceVideoSubType == MFVideoFormat_RGB32) ||
					(SourceVideoSubType == MFVideoFormat_ARGB32);

				if (Uncompressed)
				{
					// Note: MFVideoFormat_RGB32 tends to require resolutions that are multiple of 16 preventing 1920x1080 from working
					HResult = InputType->SetGUID(MF_MT_SUBTYPE, MFVideoFormat_RGB32);
					check(SUCCEEDED(HResult));

					VideoTrackFormat.SampleFormat = EMediaTextureSampleFormat::CharBMP;
					VideoTrackFormat.BufferDim = VideoTrackFormat.OutputDim;
					VideoTrackFormat.BufferStride = VideoTrackFormat.OutputDim.X * 4;
				}
				else
				{
					HResult = InputType->SetGUID(MF_MT_SUBTYPE, MFVideoFormat_YUY2);
					check(SUCCEEDED(HResult));

					VideoTrackFormat.SampleFormat = EMediaTextureSampleFormat::CharYUY2;

					int32 AlignedOutputX = VideoTrackFormat.OutputDim.X;

					if ((SourceVideoSubType == MFVideoFormat_H264) || (SourceVideoSubType == MFVideoFormat_H264_ES))
					{
						AlignedOutputX = Align(AlignedOutputX, 16);
					}

					int32 SampleStride = AlignedOutputX * 2; // 2 bytes per pixel

					if (SampleStride < 0)
					{
						SampleStride = -SampleStride;
					}

					VideoTrackFormat.BufferDim = FIntPoint(AlignedOutputX / 2, VideoTrackFormat.OutputDim.Y); // 2 pixels per texel
					VideoTrackFormat.BufferStride = SampleStride;
				}
			}

			HResult = InputType->SetUINT32(MF_MT_ALL_SAMPLES_INDEPENDENT, TRUE);
			check(SUCCEEDED(HResult));
			HResult = MFCreateSampleGrabberSinkActivate(InputType, SampleGrabberCallback, &SinkActivate);

			check(SUCCEEDED(HResult));
			SAFE_RELEASE(InputType);

			SAFE_RELEASE(OutputType);

			OutDimensions = FIntPoint(Width, Height);
		}

		SAFE_RELEASE(Handler);
	}

	IMFTopologyNode* SourceNode = NULL;
	{
		HResult = MFCreateTopologyNode(MF_TOPOLOGY_SOURCESTREAM_NODE, &SourceNode);
		check(SUCCEEDED(HResult));
		HResult = SourceNode->SetUnknown(MF_TOPONODE_SOURCE, MediaSource);
		check(SUCCEEDED(HResult));
		HResult = SourceNode->SetUnknown(MF_TOPONODE_PRESENTATION_DESCRIPTOR, PresentationDesc);
		check(SUCCEEDED(HResult));
		HResult = SourceNode->SetUnknown(MF_TOPONODE_STREAM_DESCRIPTOR, StreamDesc);
		check(SUCCEEDED(HResult));
		HResult = Topology->AddNode(SourceNode);
		check(SUCCEEDED(HResult));
	}

	IMFTopologyNode* OutputNode = NULL;
	{
		HResult = MFCreateTopologyNode(MF_TOPOLOGY_OUTPUT_NODE, &OutputNode);
		check(SUCCEEDED(HResult));
		HResult = OutputNode->SetObject(SinkActivate);
		check(SUCCEEDED(HResult));
		HResult = OutputNode->SetUINT32(MF_TOPONODE_STREAMID, 0);
		check(SUCCEEDED(HResult));
		HResult = OutputNode->SetUINT32(MF_TOPONODE_NOSHUTDOWN_ON_REMOVE, 0);
		check(SUCCEEDED(HResult));
		HResult = Topology->AddNode(OutputNode);
		check(SUCCEEDED(HResult));
	}

	HResult = SourceNode->ConnectOutput(0, OutputNode, 0);
	check(SUCCEEDED(HResult));

	SAFE_RELEASE(SourceNode);
	SAFE_RELEASE(OutputNode);
	SAFE_RELEASE(SinkActivate);

	return OutDimensions;
}



#if _MSC_VER == 1900
#pragma warning(push)
#pragma warning(disable:4838)
#endif // _MSC_VER == 1900
STDMETHODIMP FSampleGrabberCallback::QueryInterface(REFIID RefID, void** Object)
{
	static const QITAB QITab[] =
	{
		QITABENT(FSampleGrabberCallback, IMFSampleGrabberSinkCallback),
		QITABENT(FSampleGrabberCallback, IMFClockStateSink),
		{ 0 }
	};
	return QISearch(this, QITab, RefID, Object);
}
#if _MSC_VER == 1900
#pragma warning(pop)
#endif // _MSC_VER == 1900

STDMETHODIMP_(ULONG) FSampleGrabberCallback::AddRef()
{
	return FPlatformAtomics::InterlockedIncrement(&RefCount);
}

STDMETHODIMP_(ULONG) FSampleGrabberCallback::Release()
{
    int32 Ref = FPlatformAtomics::InterlockedDecrement(&RefCount);
	if (Ref == 0)
	{
		delete this;
	}

	return Ref;
}

STDMETHODIMP FSampleGrabberCallback::OnProcessSample(REFGUID MajorMediaType, DWORD SampleFlags, LONGLONG SampleTime, LONGLONG SampleDuration, const BYTE* SampleBuffer, DWORD SampleSize)
{
	if (VideoSampleReady.GetValue() == 0)
	{
		TextureData.SetNum(SampleSize, EAllowShrinking::No);
		if (SampleSize > 0)
		{
			FMemory::Memcpy(TextureData.GetData(), SampleBuffer, SampleSize);
		}

		VideoSampleReady.Set(1);
	}

	return S_OK;
}

bool FSampleGrabberCallback::GetIsSampleReadyToUpdate() const
{
	return VideoSampleReady.GetValue() != 0;
}

void FSampleGrabberCallback::SetNeedNewSample()
{
	VideoSampleReady.Set(0);
}


#include "Windows/HideWindowsPlatformTypes.h"

