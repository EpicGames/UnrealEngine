// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MoviePlayer.h"
#include "IMediaTextureSample.h"

#include "Windows/AllowWindowsPlatformTypes.h"

THIRD_PARTY_INCLUDES_START
#include <mfapi.h>
#include <mfidl.h>
THIRD_PARTY_INCLUDES_END

DECLARE_LOG_CATEGORY_EXTERN(LogWindowsMoviePlayer, Log, All);


/** The Movie Streamer is what is registered to the global movie player for Windows */
class FMediaFoundationMovieStreamer : public IMovieStreamer
{
public:
	FMediaFoundationMovieStreamer();
	~FMediaFoundationMovieStreamer();

	/** IMovieStreamer interface */
	virtual bool Init(const TArray<FString>& MoviePaths, TEnumAsByte<EMoviePlaybackType> inPlaybackType) override;
	virtual void ForceCompletion() override;
	virtual bool Tick(FRHICommandListBase& RHICmdList, float DeltaTime) override;
	virtual TSharedPtr<class ISlateViewport> GetViewportInterface() override
	{
		return MovieViewport;
	}
	virtual float GetAspectRatio() const override
	{
		return (float)MovieViewport->GetSize().X / (float)MovieViewport->GetSize().Y;
	}
	virtual void Cleanup() override;

	virtual FString GetMovieName() override;
	virtual bool IsLastMovieInPlaylist() override;

	FOnCurrentMovieClipFinished OnCurrentMovieClipFinishedDelegate;
	virtual FOnCurrentMovieClipFinished& OnCurrentMovieClipFinished() override { return OnCurrentMovieClipFinishedDelegate; }

	virtual FTextureRHIRef GetTexture() override { return Texture.IsValid() ? Texture->GetRHIRef() : nullptr; }

private:
	/** Opens up the next movie in the movie path queue */
	bool OpenNextMovie();
	/** Closes the currently running video */
	void CloseMovie();
	/** Cleans up rendering resources once movies are done playing */
	void CleanupRenderingResources();
	/** Use shader to generate final image */
	void ConvertSample(FRHICommandList& RHICmdList);

private:
	/** A list of all the stored movie paths we have enqueued for playing */
	TArray<FString> StoredMoviePaths;

	/** if true, this sequence will loop when finsihed */
	TEnumAsByte<EMoviePlaybackType> PlaybackType;

	// The index in to the playlist (StoredMoviePaths) that is currently playing
	int32 MovieIndex;

	/** Texture and viewport data for displaying to Slate */
	TArray<uint8> TextureData;
	TSharedPtr<FMovieViewport> MovieViewport;
	TSharedPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe> Texture;

	/**
	 * List of textures pending deletion, need to keep this list because we can't immediately
	 * destroy them since they could be getting used on the main thread
	 */
	TArray<TSharedPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe>> TextureFreeList;

	/** Input texture for conversion shader */
	TRefCountPtr<FRHITexture> InputTarget;

	/** The video player and sample grabber for use of Media Foundation */
	class FVideoPlayer* VideoPlayer;
    class FSampleGrabberCallback* SampleGrabberCallback;

	/** Defaults to true, modified to false if video fails to play allowing a retry without sound */
	bool bUseSound;
};

/** Video track details */
struct FMovieTrackFormat
{
public:
	FMovieTrackFormat() :
		BufferDim(FIntPoint(0, 0)),
		BufferStride(0),
		OutputDim(FIntPoint(0, 0)),
		SampleFormat(EMediaTextureSampleFormat::Undefined)
	{
	}

	FIntPoint BufferDim;
	uint32 BufferStride;
	FIntPoint OutputDim;
	EMediaTextureSampleFormat SampleFormat;
};

/** The video player is the class which handles all the loading and playing of videos */
class FVideoPlayer : public IMFAsyncCallback
{
public:
	FVideoPlayer()
		: RefCount(1)
		, MediaSession(NULL)
		, MediaSource(NULL)
		, MovieIsFinished(0)
		, CloseIsPosted(0)
		, bFailedToCreateMediaSink(false)
	{
	}

	virtual ~FVideoPlayer()
	{
		check(MediaSession == NULL);
		Shutdown();
	}
	
	// IUnknown interface
	STDMETHODIMP QueryInterface(REFIID RefID, void** Object);
	STDMETHODIMP_(ULONG) AddRef();
	STDMETHODIMP_(ULONG) Release();

	// IMFAsyncCallback interface
	STDMETHODIMP GetParameters(unsigned long*, unsigned long*) {return E_NOTIMPL;}
	STDMETHODIMP Invoke(IMFAsyncResult* AsyncResult);

	/** Opens the specified file, and returns the video dimensions */
	FIntPoint OpenFile(const FString& FilePath, class FSampleGrabberCallback* SampleGrabberCallback, bool bUseSound);
	/** Starts the video player playback */
	void StartPlayback();
	/** Shuts the video player down, destroying all threads with it */
	void Shutdown();

	/** True if the movie is still playing and rendering frames */
	bool MovieIsRunning() const {return MovieIsFinished.GetValue() == 0;}
	
	const FMovieTrackFormat& GetVideoTrackFormat() const { return VideoTrackFormat; }

	/** True when there was an error related to creating a media sink (MESessionTopologySet event received with status MF_E_CANNOT_CREATE_SINK) */
	bool FailedToCreateMediaSink() const { return bFailedToCreateMediaSink; }

private:
	/** Sets up the topology of all the nodes in the media session, returning the video dimensions */
	FIntPoint SetPlaybackTopology(class FSampleGrabberCallback* SampleGrabberCallback, bool bUseSound);
	/** Adds a single audio or video stream to the passed in topology, returning video dimensions if applicable */
	FIntPoint AddStreamToTopology(IMFTopology* Topology, IMFPresentationDescriptor* PresentationDesc, IMFStreamDescriptor* StreamDesc, class FSampleGrabberCallback* SampleGrabberCallback, bool bUseSound);

private:
	/** Media Foundation boilerplate */
	int32 RefCount;

	/** The media session which handles all playback */
	IMFMediaSession* MediaSession;
	/** The source, which reads in the data from the file */
	IMFMediaSource* MediaSource;

	/** This counter lets the ticking thread know that the current movie finished */
	FThreadSafeCounter MovieIsFinished;
	/** This counter locks the ticking thread while all Media Foundation threads shutdown */
	FThreadSafeCounter CloseIsPosted;

	/* Format of video track */
	FMovieTrackFormat VideoTrackFormat;

	/* Caches the audio device from the HMD. */
	FString HMDAudioOutputDevice;

	/** True when there was an error related to creating a media sink (MESessionTopologySet event received with status MF_E_CANNOT_CREATE_SINK) */
	bool bFailedToCreateMediaSink;
};


/** The sample grabber callback is for pulling frames off the video stream to render to texture */
class FSampleGrabberCallback : public IMFSampleGrabberSinkCallback
{
public:
	FSampleGrabberCallback(TArray<uint8>& InTextureData)
		: RefCount(1)
		, TextureData(InTextureData) 
	{
	}

	virtual ~FSampleGrabberCallback()
	{
	}


	// IUnknown interface
	STDMETHODIMP QueryInterface(REFIID RefID, void** Object);
	STDMETHODIMP_(ULONG) AddRef();
	STDMETHODIMP_(ULONG) Release();

	// IMFClockStateSink interface
	STDMETHODIMP OnClockStart(MFTIME SystemTime, LONGLONG llClockStartOffset) {return S_OK;}
	STDMETHODIMP OnClockStop(MFTIME SystemTime) {return S_OK;}
	STDMETHODIMP OnClockPause(MFTIME SystemTime) {return S_OK;}
	STDMETHODIMP OnClockRestart(MFTIME SystemTime) {return S_OK;}
	STDMETHODIMP OnClockSetRate(MFTIME SystemTime, float flRate) {return S_OK;}
	
	// IMFSampleGrabberSinkCallback interface
	STDMETHODIMP OnSetPresentationClock(IMFPresentationClock* Clock) {return S_OK;}
	STDMETHODIMP OnProcessSample(REFGUID MajorMediaType, DWORD SampleFlags,
        LONGLONG SampleTime, LONGLONG SampleDuration, const BYTE* SampleBuffer,
        DWORD SampleSize);
	STDMETHODIMP OnShutdown() {return S_OK;}

	/** True if we have a new sample for readback */
	bool GetIsSampleReadyToUpdate() const;

	/** Tells this callback that we need a new sample to read back */
	void SetNeedNewSample();

private:
	/** Media Foundation boilerplate */
	int32 RefCount;

	/** Counter which determines when a sample can be safely read back */
	FThreadSafeCounter VideoSampleReady;
	/** The texture data sample we read back to */
	TArray<uint8>& TextureData;
};

#include "Windows/HideWindowsPlatformTypes.h"
