// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class WindowsMoviePlayer : ModuleRules
	{
        public WindowsMoviePlayer(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePaths.AddRange(
				new string[] {
				}
				);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"HeadMountedDisplay",
					"Media",
				});

			PrivateIncludePaths.AddRange(
				new string[] {
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject", // @todo Mac: for some reason CoreUObject and Engine are needed to link in debug on Mac
					"Engine", // @todo Mac: for some reason CoreUObject and Engine are needed to link in debug on Mac
                    "InputCore",
                    "MoviePlayer",
                    "RenderCore",
                    "RHI",
                    "SlateCore",
                    "Slate"
				}
				);

			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
				}
				);

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				PublicDelayLoadDLLs.Add("shlwapi.dll");
				PublicDelayLoadDLLs.Add("mf.dll");
				PublicDelayLoadDLLs.Add("mfplat.dll");
				PublicDelayLoadDLLs.Add("mfplay.dll");
				PublicDelayLoadDLLs.Add("mfuuid.dll");
			}
		}
	}
}
