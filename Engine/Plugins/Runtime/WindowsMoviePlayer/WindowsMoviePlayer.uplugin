{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Windows Movie Player",
	"Description": "Windows Specific Movie Player using Media Foundation",
	"Category": "Media Players",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "WindowsMoviePlayer",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PreLoadingScreen",
			"PlatformAllowList": [
				"Win64"
			]
		}
	],
	"IsExperimentalVersion": false
}